{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Material Asset Wizard",
	"Description": "Enables new factories for Materials and Material Instances.\nRequires setting Tagged Asset Browser Configuration properties under Editor Preferences -> Material Editor Settings.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "MaterialAssetWizard",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}