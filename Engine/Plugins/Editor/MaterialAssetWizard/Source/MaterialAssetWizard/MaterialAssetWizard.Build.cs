// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MaterialAssetWizard : ModuleRules
{
	public MaterialAssetWizard(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"UserAssetTagsEditor",
				"UnrealEd"
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"MaterialEditor"
				// ... add private dependencies that you statically link with here ...	
			}
			);
	}
}
