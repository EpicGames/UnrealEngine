// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialAssetWizardModule.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Widgets/SUserAssetTagBrowserSelectedAssetDetails.h"
#include "Widgets/Text/STextBlock.h"
#include "Materials/Material.h"

#define LOCTEXT_NAMESPACE "MaterialAssetWizardModule"

void FMaterialAssetWizardModule::StartupModule()
{
	// Materials
	{
		FAssetDetailsDisplayInfo MaterialInfo;

		// Example
#if 0
		// Number of emitters
		FDisplayedPropertyData NumEmittersProperty;
		NumEmittersProperty.ShouldDisplayPropertyDelegate = FDisplayedPropertyData::FShouldDisplayProperty::CreateLambda([](const FAssetData& AssetData) -> bool
		{
			return AssetData.FindTag("NumEmitters");
		});
		NumEmittersProperty.NameWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
			.Text(LOCTEXT("NumberOfEmitters", "Number of Emitters"));
		});
		NumEmittersProperty.ValueWidgetDelegate =  FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			int32 NumEmitters = INDEX_NONE;
			if(AssetData.GetTagValue("NumEmitters", NumEmitters))
			{
				return SNew(STextBlock).Text(FText::AsNumber(NumEmitters));
			}
	
			return SNullWidget::NullWidget;
		});
		
		MaterialInfo.DisplayedProperties =
		{
			NumEmittersProperty
		};
		
		MaterialInfo.GetDescriptionDelegate = FAssetDetailsDisplayInfo::FGetDescription::CreateLambda([](const FAssetData& AssetData)
		{
			FName DescriptionTagName("TemplateAssetDescription");
			if(AssetData.FindTag(DescriptionTagName))
			{
				FString Description;
				AssetData.GetTagValue(DescriptionTagName, Description);
				return FText::FromString(Description);
			}
		
			return FText::GetEmpty();
		});
#endif
		
		FTaggedAssetBrowserDetailsDisplayDatabase::RegisterClass(UMaterial::StaticClass(), MaterialInfo);
	}

	// Material Instances
	{
		FAssetDetailsDisplayInfo MaterialInstanceInfo;

		// Example
#if 0
		// Number of emitters
		FDisplayedPropertyData NumEmittersProperty;
		NumEmittersProperty.ShouldDisplayPropertyDelegate = FDisplayedPropertyData::FShouldDisplayProperty::CreateLambda([](const FAssetData& AssetData) -> bool
		{
			return AssetData.FindTag("NumEmitters");
		});
		NumEmittersProperty.NameWidgetDelegate = FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			return SNew(STextBlock)
			.Text(LOCTEXT("NumberOfEmitters", "Number of Emitters"));
		});
		NumEmittersProperty.ValueWidgetDelegate =  FDisplayedPropertyData::FGenerateWidget::CreateLambda([](const FAssetData& AssetData) -> TSharedRef<SWidget>
		{
			int32 NumEmitters = INDEX_NONE;
			if(AssetData.GetTagValue("NumEmitters", NumEmitters))
			{
				return SNew(STextBlock).Text(FText::AsNumber(NumEmitters));
			}
	
			return SNullWidget::NullWidget;
		});
		
		MaterialInstanceInfo.DisplayedProperties =
		{
			NumEmittersProperty
		};
		
		MaterialInstanceInfo.GetDescriptionDelegate = FAssetDetailsDisplayInfo::FGetDescription::CreateLambda([](const FAssetData& AssetData)
		{
			FName DescriptionTagName("TemplateAssetDescription");
			if(AssetData.FindTag(DescriptionTagName))
			{
				FString Description;
				AssetData.GetTagValue(DescriptionTagName, Description);
				return FText::FromString(Description);
			}
		
			return FText::GetEmpty();
		});
#endif
		
		FTaggedAssetBrowserDetailsDisplayDatabase::RegisterClass(UMaterialInstanceConstant::StaticClass(), MaterialInstanceInfo);
	}
}

void FMaterialAssetWizardModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMaterialAssetWizardModule, MaterialAssetWizard)