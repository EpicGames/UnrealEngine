// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialFactoryWithAssetWizard.h"

#include "EditorDirectories.h"
#include "MaterialEditingLibrary.h"
#include "MaterialEditorSettings.h"
#include "Assets/TaggedAssetBrowserConfiguration.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "Widgets/STaggedAssetBrowserAssetFactoryWindow.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"

#define LOCTEXT_NAMESPACE "MaterialFactoryWithAssetWizard"

bool UMaterialFactoryWithAssetWizardBase::ConfigureProperties()
{
	if (SupportedClass == nullptr)
	{
		return false;
	}
	
	// Needs to return true to allow async path
	return true;
}

bool UMaterialFactoryWithAssetWizardBase::ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete, FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled)
{	
	UObject* ConfigUObject = ConfigurationPath.TryLoad();
	if(UTaggedAssetBrowserConfiguration* ConfigurationAsset = Cast<UTaggedAssetBrowserConfiguration>(ConfigUObject))
	{
		SWindow::FArguments WindowArgs = CreateWindowArguments();
		STaggedAssetBrowser::FArguments AssetBrowserArgs = CreateTaggedAssetBrowserArguments();
		
		STaggedAssetBrowserWindow::FArguments AssetBrowserWindowArgs;
		AssetBrowserWindowArgs.AssetBrowserArgs(AssetBrowserArgs);
		AssetBrowserWindowArgs.WindowArgs(WindowArgs);
		
		STaggedAssetBrowserAssetFactoryWindow::FAsyncFactoryArguments AsyncArguments;
		AsyncArguments.Factory.Reset(this);
		AsyncArguments.OnAssetsActivated = FOnAssetsActivated::CreateUObject(this, &UMaterialFactoryWithAssetWizardBase::OnAssetsActivated);
		AsyncArguments.OnFactoryConfigurePropertiesComplete = OnComplete;
		AsyncArguments.OnFactoryConfigurePropertiesCancelled = OnCancelled;
		
		TSharedRef<STaggedAssetBrowserAssetFactoryWindow> CreateAssetBrowserWindow = SNew(STaggedAssetBrowserAssetFactoryWindow, *ConfigurationAsset, *SupportedClass.Get())
			.AssetBrowserWindowArgs(AssetBrowserWindowArgs)
			.AsyncFactoryArguments(AsyncArguments);
		
		FSlateApplication::Get().AddWindow(CreateAssetBrowserWindow, true);

		return true;
	}

	OnComplete.ExecuteIfBound(this);
	return true;
}

void UMaterialFactoryWithAssetWizardBase::OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type Method)
{
	if(AssetData.Num() == 1)
	{
		FAssetData SelectedAsset = AssetData[0];
		ensure(SelectedAsset.GetClass() == UMaterial::StaticClass());
		
		UMaterial* MaterialAsset = Cast<UMaterial>(SelectedAsset.GetAsset());
			
		BaseMaterial = MaterialAsset;
	}
}

SWindow::FArguments UMaterialFactoryWithAssetWizardBase::CreateWindowArguments() const
{
	SWindow::FArguments Result;
	Result.SupportsMaximize(false);
	Result.SupportsMinimize(false);
	Result.ClientSize(FVector2D(1400, 750));
	Result.SizingRule(ESizingRule::UserSized);
	Result.Title(WindowTitle);
		
	return Result;
}

STaggedAssetBrowser::FArguments UMaterialFactoryWithAssetWizardBase::CreateTaggedAssetBrowserArguments() const
{
	FDefaultDetailsTabConfiguration DefaultDetailsTabConfiguration;
	DefaultDetailsTabConfiguration.bUseDefaultDetailsTab = true;
	DefaultDetailsTabConfiguration.EmptySelectionMessage = EmptySelectionMessage;
		
	STaggedAssetBrowser::FInterfaceOverrideProfiles OverrideProfiles;
	OverrideProfiles.AssetViewOptionsProfileName = FName("TaggedAssetBrowser");
	OverrideProfiles.DefaultFilterMenuExpansion = EAssetTypeCategories::Materials;
	OverrideProfiles.FilterBarSaveName = FName("TaggedAssetBrowser.MaterialFactory");

	FString PackagePath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
	FString AssetName = "TmpAsset";
	FName PackageName = FName(PackagePath + "/" + AssetName);
	FAssetData PseudoReferencingAsset(PackageName, FName(PackagePath), FName(AssetName), UObject::StaticClass()->GetClassPathName());
	
	STaggedAssetBrowser::FArguments Result;
	Result.AvailableClasses({ UMaterial::StaticClass() });
	Result.DefaultDetailsTabConfiguration(DefaultDetailsTabConfiguration);
	Result.InterfaceOverrideProfiles(OverrideProfiles);
	Result.AdditionalReferencingAssets({ PseudoReferencingAsset });

	return Result;
}

UMaterialFactoryWithAssetWizard::UMaterialFactoryWithAssetWizard()
{
	SupportedClass = UMaterial::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedWorkflows = (uint8) (EFactoryCreateWorkflow::Default | EFactoryCreateWorkflow::Asynchronous);

	ConfigurationPath = GetDefault<UMaterialEditorSettings>()->MaterialWizardConfiguration;
	WindowTitle = LOCTEXT("MaterialAssetBrowserWindowTitle", "Create Material - Select another material as a base");
	EmptySelectionMessage = LOCTEXT("EmptyMaterialFactorySelectionMessage", "Select a material as a starting point for your new material.\n");
}

UObject* UMaterialFactoryWithAssetWizard::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UMaterial::StaticClass()));

	const UMaterialEditorSettings* Settings = GetDefault<UMaterialEditorSettings>();
	check(Settings);

	UMaterial* NewMaterial;

	// Duplicate the selected material
	if (BaseMaterial.IsValid())
	{
		NewMaterial = Cast<UMaterial>(StaticDuplicateObject(BaseMaterial.Get(), InParent, Name, Flags, Class));
	}
	// Or create an empty material as a fallback
	else
	{
		NewMaterial = NewObject<UMaterial>(InParent, Class, Name, Flags | RF_Transactional);
	}

	NewMaterial->CacheShaders(EMaterialShaderPrecompileMode::Background);

	return NewMaterial;
}

bool UMaterialFactoryWithAssetWizard::ShouldShowInNewMenu() const
{
	return UFactory::ShouldShowInNewMenu() && GetDefault<UMaterialEditorSettings>()->MaterialWizardConfiguration != nullptr;
}

UMaterialInstanceConstantFactoryWithAssetWizard::UMaterialInstanceConstantFactoryWithAssetWizard()
{
	SupportedClass = UMaterialInstanceConstant::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedWorkflows = (uint8) (EFactoryCreateWorkflow::Default | EFactoryCreateWorkflow::Asynchronous);

	ConfigurationPath = GetDefault<UMaterialEditorSettings>()->MaterialInstanceWizardConfiguration;
	WindowTitle = LOCTEXT("MaterialInstanceAssetBrowserWindowTitle", "Create Material Instance - Select another Material as a base");
	EmptySelectionMessage = LOCTEXT("EmptyMaterialInstanceFactorySelectionMessage", "Select a material as a starting point for your new material instance.\n");
}

UObject* UMaterialInstanceConstantFactoryWithAssetWizard::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags,	UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UMaterialInstanceConstant::StaticClass()));

	UMaterialInstanceConstant* NewMaterialInstance = NewObject<UMaterialInstanceConstant>(InParent, Class, Name, Flags | RF_Transactional);
	
	if (BaseMaterial.IsValid())
	{
		UMaterialEditingLibrary::SetMaterialInstanceParent(NewMaterialInstance, BaseMaterial.Get());
	}

	return NewMaterialInstance;
}

bool UMaterialInstanceConstantFactoryWithAssetWizard::ShouldShowInNewMenu() const
{
	return UFactory::ShouldShowInNewMenu() && GetDefault<UMaterialEditorSettings>()->MaterialInstanceWizardConfiguration != nullptr;
}

#undef LOCTEXT_NAMESPACE
