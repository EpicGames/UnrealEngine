// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "Widgets/STaggedAssetBrowser.h"
#include "MaterialFactoryWithAssetWizard.generated.h"

class STaggedAssetBrowserAssetFactoryWindow;
class UMaterial;

/** Abstract base factory that summons material wizard. */
UCLASS(Abstract, MinimalAPI)
class UMaterialFactoryWithAssetWizardBase : public UFactory
{
	GENERATED_BODY()

public:
	UMaterialFactoryWithAssetWizardBase() = default;
	
	virtual bool ConfigureProperties() override;
	virtual bool ConfigurePropertiesAsync(FOnFactoryConfigurePropertiesAsyncComplete OnComplete, FOnFactoryConfigurePropertiesAsyncCancelled OnCancelled) override;
	virtual bool ShouldShowInNewMenu() const override { return false; }

private:
	void OnAssetsActivated(const TArray<FAssetData>& AssetData, EAssetTypeActivationMethod::Type Method);
	
	SWindow::FArguments CreateWindowArguments() const;
	STaggedAssetBrowser::FArguments CreateTaggedAssetBrowserArguments() const;

protected:
	/** The path to the tagged asset browser configuration asset. Needs to be valid. */
	UPROPERTY()
	FSoftObjectPath ConfigurationPath;
	
	UPROPERTY()
	FText WindowTitle;

	UPROPERTY()
	FText EmptySelectionMessage;

	/** The selected base material, if any. */
	TWeakObjectPtr<UMaterial> BaseMaterial;
};

/** Concrete material factory that will create a new material based on the selected material, or an empty one. */
UCLASS(MinimalAPI)
class UMaterialFactoryWithAssetWizard : public UMaterialFactoryWithAssetWizardBase
{
	GENERATED_BODY()

public:
	UMaterialFactoryWithAssetWizard();
	
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
};

/** Concrete material instance constant factory that will create a new material instance based on the selected material, or an empty one. */
UCLASS(MinimalAPI)
class UMaterialInstanceConstantFactoryWithAssetWizard : public UMaterialFactoryWithAssetWizardBase
{
	GENERATED_BODY()

public:
	UMaterialInstanceConstantFactoryWithAssetWizard();
	
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
};
